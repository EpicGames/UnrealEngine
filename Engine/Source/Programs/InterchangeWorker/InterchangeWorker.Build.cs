// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class InterchangeWorker : ModuleRules
{
	public InterchangeWorker(ReadOnlyTargetRules Target) : base(Target)
	{
		if (Target.Platform != UnrealTargetPlatform.Win64 &&
			Target.Platform != UnrealTargetPlatform.Linux &&
			Target.Platform != UnrealTargetPlatform.Mac)
		{
			throw new BuildException("InterchangeWorker program do not support target platform {0}.", Target.Platform.ToString());
		}

		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"Core",
				"CoreUObject",
				"InterchangeCore",
				"InterchangeDispatcher",
				"InterchangeFbxParser",
				"InterchangeNodes",
				"Json",
				"Projects",
				"Sockets",
			}
		);

		AddEngineThirdPartyPrivateStaticDependencies(Target,
			new string[]
			{
				"FBX"
			}
		);
	}
}
