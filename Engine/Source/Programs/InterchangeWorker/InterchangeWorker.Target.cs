// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
[SupportedConfigurations(UnrealTargetConfiguration.Debug, UnrealTargetConfiguration.Development, UnrealTargetConfiguration.Shipping)]
public class InterchangeWorkerTarget : TargetRules
{
	public InterchangeWorkerTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LaunchModuleName = "InterchangeWorker";
		SolutionDirectory = "Programs/InterchangeWorker";

		// Lean and mean
		bBuildDeveloperTools = false;

		// Editor-only data, however, is needed
		bBuildWithEditorOnlyData = false;

		// Currently this app is not linking against the engine, so we'll compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = true;
		bCompileAgainstApplicationCore = true;

		// This is a console application, not a Windows app (sets entry point to main(), instead of WinMain())
		bIsBuildingConsoleApplication = true;

		bLegalToDistributeBinary = true;

		bCompileICU = false;
		bCompileCEF3 = false;

		GlobalDefinitions.Add("USE_LOCALIZED_PACKAGE_CACHE=0");

	}
}
