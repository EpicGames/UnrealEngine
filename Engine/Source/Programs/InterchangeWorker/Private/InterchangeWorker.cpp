// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeWorker.h"

#include "HAL/PlatformProcess.h"
#include "InterchangeResultsContainer.h"
#include "InterchangeWorkerImpl.h"
#include "RequiredProgramMainCPPInclude.h"
#include "UObject/UObjectGlobals.h"

IMPLEMENT_APPLICATION(InterchangeWorker, "InterchangeWorker");
DEFINE_LOG_CATEGORY(LogInterchangeWorker);

#define EXIT_MISSING_CORETECH_MODULE 2

void GetParameter(int32 Argc, TCHAR* Argv[], const FString& InParam, FString& OutValue)
{
	OutValue.Empty();
	for (int32 Index = 1; Index < Argc; Index++)
	{
		if (! FCString::Strcmp(*InParam, Argv[Index]))
		{
			if (Index + 1 < Argc)
			{
				OutValue = Argv[Index + 1];
			}
		}
	}
}

bool HasParameter(int32 Argc, TCHAR* Argv[], const FString& InParam)
{
	for (int32 Index = 1; Index < Argc; Index++)
	{
		if (!FCString::Strcmp(*InParam, Argv[Index]))
		{
			return true;
		}
	}
	return false;
}

int32 Main(int32 Argc, TCHAR * Argv[])
{
	UE_SET_LOG_VERBOSITY(LogInterchangeWorker, Verbose);

	FString ResultFolder, ServerPID, ServerPort, InterchangeDispatcherVersion;
	GetParameter(Argc, Argv, "-ServerPID", ServerPID);
	GetParameter(Argc, Argv, "-ServerPort", ServerPort);
	GetParameter(Argc, Argv, "-InterchangeDispatcherVersion", InterchangeDispatcherVersion);
	GetParameter(Argc, Argv, "-ResultFolder", ResultFolder);

	int32 Major = 0;
	int32 Minor = 0;
	int32 Patch = 0;
	bool bLWCDisabled = true;

	FString WorkerVersionError;
	if (!UE::Interchange::DispatcherCommandVersion::FromString(InterchangeDispatcherVersion, Major, Minor, Patch, bLWCDisabled))
	{
		WorkerVersionError = TEXT("Incompatible interchange dispatcher version string command argument.");
		WorkerVersionError += TEXT(" Editor Version: [");
		WorkerVersionError += InterchangeDispatcherVersion;
		WorkerVersionError += TEXT("] Worker Version: [");
		WorkerVersionError += UE::Interchange::DispatcherCommandVersion::ToString();
		WorkerVersionError += TEXT("]");
		UE_LOGF(LogInterchangeWorker, Error, "%ls", *WorkerVersionError);
	}

 	if (!UE::Interchange::DispatcherCommandVersion::IsAPICompatible(Major, Minor, Patch, bLWCDisabled))
 	{
		WorkerVersionError = TEXT("Incompatible interchange dispatcher API version. Please recompile InterchangeWorker target.");
		WorkerVersionError += TEXT(" Editor Version: [");
		WorkerVersionError += InterchangeDispatcherVersion;
		WorkerVersionError += TEXT("] Worker Version: [");
		WorkerVersionError += UE::Interchange::DispatcherCommandVersion::ToString();
		WorkerVersionError += TEXT("]");
 		UE_LOGF(LogInterchangeWorker, Error, "%ls", *WorkerVersionError);
 	}

	//FInterchangeWorkerImpl Scope
	{
		FInterchangeWorkerImpl Worker(FCString::Atoi(*ServerPID), FCString::Atoi(*ServerPort), ResultFolder);
		Worker.Run(WorkerVersionError);
	}

	if (!WorkerVersionError.IsEmpty())
	{
		return EXIT_FAILURE;
	}

	return EXIT_SUCCESS;
}

#if PLATFORM_WINDOWS
int32 Filter(uint32 Code, struct _EXCEPTION_POINTERS *Ep)
{
	return EXCEPTION_EXECUTE_HANDLER;
}
#endif //#if PLATFORM_WINDOWS

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	int32 ReturnCode = 0;
	GEngineLoop.PreInit(ArgC, ArgV);

#if PLATFORM_WINDOWS
	SetErrorMode(SEM_FAILCRITICALERRORS | SEM_NOGPFAULTERRORBOX);
	_set_abort_behavior(0, _WRITE_ABORT_MSG);
	__try
	{
		ReturnCode = Main(ArgC, ArgV);
	}
	__except (Filter(GetExceptionCode(), GetExceptionInformation()))
	{
		ReturnCode = EXIT_FAILURE;
	}
#else // PLATFORM_WINDOWS
	ReturnCode = Main(ArgC, ArgV);
#endif // PLATFORM_WINDOWS

	FEngineLoop::AppPreExit();
	FModuleManager::Get().UnloadModulesAtShutdown();
	FEngineLoop::AppExit();
	FPlatformMisc::RequestExit(false);

	return ReturnCode;
}
