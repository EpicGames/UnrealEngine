// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeWorkerImpl.h"

#include "Async/Async.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "InterchangeCommands.h"
#include "InterchangeDispatcherConfig.h"
#include "FBX/InterchangeDispatcherFBXTasks.h"
#include "InterchangeFbxParser.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "SocketSubsystem.h"
#include "Sockets.h"

using namespace UE::Interchange;

FInterchangeWorkerImpl::FInterchangeWorkerImpl(int32 InServerPID, int32 InServerPort, FString& InResultFolder)
	: ServerPID(InServerPID)
	, ServerPort(InServerPort)
	, PingStartCycle(0)
	, ResultFolder(InResultFolder)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	FPaths::NormalizeDirectoryName(ResultFolder);
	if (!PlatformFile.DirectoryExists(*ResultFolder))
	{
		PlatformFile.CreateDirectory(*ResultFolder);
	}
}

bool FInterchangeWorkerImpl::Run(const FString& WorkerVersionError)
{
	UE_LOGF(LogInterchangeWorker, Verbose, "connect to %d...", ServerPort);
	bool bConnected = NetworkInterface.Connect(TEXT("Interchange Worker"), ServerPort, Config::ConnectTimeout_s);
	UE_LOGF(LogInterchangeWorker, Verbose, "connected to %d %ls", ServerPort, bConnected ? TEXT("OK") : TEXT("FAIL"));
	if (bConnected)
	{
		CommandIO.SetNetworkInterface(&NetworkInterface);
	}
	else
	{
		UE_LOGF(LogInterchangeWorker, Error, "Server connection failure. exit");
		return false;
	}

	const bool bVersionError = !WorkerVersionError.IsEmpty();
	if (bVersionError)
	{
		FErrorCommand ErrorCmd;
		
		UInterchangeResultError_Generic* Message = FbxParser.AddMessage<UInterchangeResultError_Generic>();
		Message->Text = FText::Format(NSLOCTEXT("InterchangeWorker", "WorkerVersionError", "Version Error: {0}"), FText::FromString(WorkerVersionError));

		ErrorCmd.ErrorMessage = Message->ToJson();
		CommandIO.SendCommand(ErrorCmd, Config::SendCommandTimeout_s);
		//We want to time out after maximum of 5 seconds
		double TimeOut = 5.0;
		while (TimeOut > 0)
		{
			if (TSharedPtr<ICommand> Command = CommandIO.GetNextCommand(0.02))
			{
				if(Command->GetType() == ECommandId::Terminate)
				{
					UE_LOGF(LogInterchangeWorker, Verbose, "Terminate command received. Exiting.");
					break;
				}
			}
			TimeOut -= 0.02;
		}
		return false;
	}

	InitiatePing();

	bool bIsRunning = true;
	while (bIsRunning)
	{
		if (TSharedPtr<ICommand> Command = CommandIO.GetNextCommand(0.02))
		{
			switch(Command->GetType())
			{
				case ECommandId::Ping:
					ProcessCommand(*StaticCast<FPingCommand*>(Command.Get()));
					break;

				case ECommandId::BackPing:
					ProcessCommand(*StaticCast<FBackPingCommand*>(Command.Get()));
					break;

				case ECommandId::RunTask:
				{
					FScopeLock Lock(&TFinishThreadCriticalSection);
					//Use a randomGuid to generate thread unique name
					int32 UniqueID = (FPlatformTime::Cycles64() & 0x00000000EFFFFFFF);
					FString ThreadName = FString(TEXT("InterchangeWorkerCommand_")) + FString::FromInt(UniqueID);
					ActiveThreads.Add(ThreadName, Async(
						EAsyncExecution::ThreadPool,
						[this, ThreadName, Command]()->bool
						{
							ProcessCommand(Command, ThreadName);
							return true;
						}
					));
				}
					break;
				case ECommandId::QueryTaskProgress:
					ProcessCommand(*StaticCast<FQueryTaskProgressCommand*>(Command.Get()));
					break;

				case ECommandId::Terminate:
					UE_LOGF(LogInterchangeWorker, Verbose, "Terminate command received. Exiting.");
					bIsRunning = false;
					break;

				case ECommandId::NotifyEndTask:
				default:
					break;
			}
		}
		else
		{
			if (bIsRunning)
			{
				bIsRunning = ServerPID == 0 ? true : FPlatformProcess::IsApplicationRunning(ServerPID);
				UE_CLOGF(!bIsRunning, LogInterchangeWorker, Error, "Worker failure: server lost");
			}
		}
		
		//Cleanup Finish threads
		{
			FScopeLock Lock(&TFinishThreadCriticalSection);
			for (const FString& ThreadName : CurrentFinishThreads)
			{
				if (TFuture<bool>* IsThreadCompleted = ActiveThreads.Find(ThreadName))
				{
					//Wait until process command is terminate
					IsThreadCompleted->Get();
				}
				ActiveThreads.Remove(ThreadName);
			}
			CurrentFinishThreads.Empty();
		}

		//Sleep 0 to avoid using too much cpu
		FPlatformProcess::Sleep(0.0f);
	}

	//Join all thread that is not terminate
	for(TPair<FString, TFuture<bool>>& ActiveThreadPair : ActiveThreads)
	{
		ActiveThreadPair.Value.Get();
	}

	UE_CLOGF(!bIsRunning, LogInterchangeWorker, Verbose, "Worker loop exit...");
	CommandIO.Disconnect(0);
	return true;
}

void FInterchangeWorkerImpl::InitiatePing()
{
	PingStartCycle = FPlatformTime::Cycles64();
	FPingCommand Ping;
	CommandIO.SendCommand(Ping, Config::SendCommandTimeout_s);
}

void FInterchangeWorkerImpl::ProcessCommand(const FPingCommand& PingCommand)
{
	FBackPingCommand BackPing;
	CommandIO.SendCommand(BackPing, Config::SendCommandTimeout_s);
}

void FInterchangeWorkerImpl::ProcessCommand(const FBackPingCommand& BackPingCommand)
{
	if (PingStartCycle)
	{
		double ElapsedTime_s = FGenericPlatformTime::ToSeconds(FPlatformTime::Cycles64() - PingStartCycle);
		UE_LOGF(LogInterchangeWorker, Verbose, "Ping %f s", ElapsedTime_s);
	}
	PingStartCycle = 0;
}

void FInterchangeWorkerImpl::ProcessCommand(const UE::Interchange::FQueryTaskProgressCommand& QueryTaskProgressCommand)
{
	FCompletedQueryTaskProgressCommand CompletedCommand;
	CompletedCommand.TaskStates.AddDefaulted(QueryTaskProgressCommand.TaskIndexes.Num());
	{
		FScopeLock Lock(&TFinishThreadCriticalSection);
		for (int32 ProgressTaskIndex = 0; ProgressTaskIndex < QueryTaskProgressCommand.TaskIndexes.Num(); ++ProgressTaskIndex)
		{
			CompletedCommand.TaskStates[ProgressTaskIndex].TaskIndex = QueryTaskProgressCommand.TaskIndexes[ProgressTaskIndex];
			//If we do not have any job to process, set the state to unknown. The caller will know we are not currently processing this task, whihc mean we never receive this task or this task is completed.
			CompletedCommand.TaskStates[ProgressTaskIndex].TaskState = ActiveThreads.Num() == 0 ? ETaskState::Unknown : ETaskState::Running;
			//TODO: implement better progress report, we currently always return 0.0
			CompletedCommand.TaskStates[ProgressTaskIndex].TaskProgress = 0.0f;
		}
	}
	CommandIO.SendCommand(CompletedCommand, Config::SendCommandTimeout_s);
}

void FInterchangeWorkerImpl::ProcessCommand(const TSharedPtr<UE::Interchange::ICommand> Command, const FString& ThreadName)
{
	if (!Command.IsValid())
	{
		UE_LOGF(LogInterchangeWorker, Error, "Process command error: The run task command is invalid");
		return;
	}
	FRunTaskCommand* RunTaskCommandPtr = StaticCast<FRunTaskCommand*>(Command.Get());
	if (!RunTaskCommandPtr)
	{
		UE_LOGF(LogInterchangeWorker, Error, "Process command error: The run task command is invalid");
		return;
	}
	const FRunTaskCommand& RunTaskCommand = *RunTaskCommandPtr;

	const FString& JsonToProcess = RunTaskCommand.JsonDescription;
	UE_LOGF(LogInterchangeWorker, Verbose, "Process %ls", *JsonToProcess);

	ETaskState ProcessResult = ETaskState::Unknown;
	//Process the json and run the task
	FString JSonResult;
	TArray<FString> JSonMessages;
	FJsonFBXLoadSourceCmd LoadSourceCommand;
	FJsonFetchAnimationQueriesCmd FetchAnimationQueries;
	FJsonFetchMeshPayloadCmd FetchMeshPayloadCommand;
	FJsonFetchPayloadCmd FetchPayloadCommand;
	//Any command FromJson function return true if the Json descibe the command
	if (LoadSourceCommand.FromJson(JsonToProcess))
	{
		//Load file command
		if (LoadSourceCommand.GetTranslatorID().Equals(FJsonFBXLoadSourceCmd::TaskName))
		{
			//We want to load an FBX file
			ProcessResult = LoadFbxFile(LoadSourceCommand, JSonResult, JSonMessages);
		}
	}
	else if (FetchAnimationQueries.FromJson(JsonToProcess))
	{
		//Load file command
		if (FetchAnimationQueries.GetTranslatorID().Equals(TEXT("FBX"), ESearchCase::IgnoreCase))
		{
			//We want to load an FBX file
			ProcessResult = FetchFbxPayload(FetchAnimationQueries, JSonResult, JSonMessages);
		}
	}
	else if (FetchMeshPayloadCommand.FromJson(JsonToProcess))
	{
		//Load file command
		if (FetchMeshPayloadCommand.GetTranslatorID().Equals(TEXT("FBX"), ESearchCase::IgnoreCase))
		{
			//We want to load an FBX file
			ProcessResult = FetchFbxPayload(FetchMeshPayloadCommand, JSonResult, JSonMessages);
		}
	}
	else if (FetchPayloadCommand.FromJson(JsonToProcess))
	{
		//Load file command
		if (FetchPayloadCommand.GetTranslatorID().Equals(TEXT("FBX"), ESearchCase::IgnoreCase))
		{
			//We want to load an FBX file
			ProcessResult = FetchFbxPayload(FetchPayloadCommand, JSonResult, JSonMessages);
		}
	}
	else
	{
		ProcessResult = ETaskState::Unknown;
	}

	FCompletedTaskCommand CompletedTask;
	CompletedTask.ProcessResult = ProcessResult;
	CompletedTask.JSonMessages = JSonMessages;
	CompletedTask.TaskIndex = RunTaskCommand.TaskIndex;
	if (CompletedTask.ProcessResult == ETaskState::ProcessOk)
	{
		CompletedTask.JSonResult = JSonResult;
	}

	UE_LOGF(LogInterchangeWorker, Verbose, "CompletedTask.Result : %d\n %ls", (int32)ProcessResult , *CompletedTask.JSonResult);
	int32 Index = 0;
	for (const FString& Message : CompletedTask.JSonMessages)
	{
		UE_LOGF(LogInterchangeWorker, Verbose, "CompletedTask.Message #%d : %ls", ++Index, *Message);
	}
	CommandIO.SendCommand(CompletedTask, Config::SendCommandTimeout_s);
	UE_LOGF(LogInterchangeWorker, Verbose, "End of Process %ls", *JsonToProcess);
	
	//Notify the main thread we are done with this thread
	{
		FScopeLock Lock(&TFinishThreadCriticalSection);
		CurrentFinishThreads.Add(ThreadName);
	}
}

ETaskState FInterchangeWorkerImpl::LoadFbxFile(const FJsonFBXLoadSourceCmd& LoadSourceCommand, FString& OutJSonResult, TArray<FString>& OutJSonMessages)
{
	ETaskState ResultState = ETaskState::Unknown;
	FString SourceFilename = LoadSourceCommand.GetSourceFilename();
	UE_LOGF(LogInterchangeWorker, Verbose, "LoadFbxFile : %ls", *SourceFilename);
	FbxParser.Reset();
	FbxParser.SetConvertSettings(LoadSourceCommand.GetDoesConvertScene(), LoadSourceCommand.GetDoesForceFrontXAxis(), LoadSourceCommand.GetDoesConvertSceneUnit(), LoadSourceCommand.GetDoesKeepFbxNamespace(), LoadSourceCommand.GetDoesUseFbxClustersForBindPoseMatrices());
	FbxParser.LoadFbxFile(SourceFilename, ResultFolder);
	FJsonLoadSourceCmd::JsonResultParser ResultParser;
	ResultParser.SetResultFilename(FbxParser.GetResultFilepath());
	OutJSonMessages = FbxParser.GetJsonLoadMessages();
	OutJSonResult = ResultParser.ToJson();
	UE_LOGF(LogInterchangeWorker, Verbose, "ResultParser : %ls", *OutJSonResult);
	ResultState = ETaskState::ProcessOk;
	return ResultState;
}

ETaskState FInterchangeWorkerImpl::FetchFbxPayload(const FJsonFetchPayloadCmd& FetchPayloadCommand, FString& OutJSonResult, TArray<FString>& OutJSonMessages)
{
	ETaskState ResultState = ETaskState::Unknown;
	FString PayloadKey = FetchPayloadCommand.GetPayloadKey();
	FbxParser.FetchPayload(PayloadKey, ResultFolder);
	FJsonLoadSourceCmd::JsonResultParser ResultParser;
	ResultParser.SetResultFilename(FbxParser.GetResultPayloadFilepath(PayloadKey));
	OutJSonMessages = FbxParser.GetJsonLoadMessages();
	OutJSonResult = ResultParser.ToJson();
	ResultState = ETaskState::ProcessOk;
	return ResultState;
}

ETaskState FInterchangeWorkerImpl::FetchFbxPayload(const FJsonFetchMeshPayloadCmd& FetchMeshPayloadCommand, FString& OutJSonResult, TArray<FString>& OutJSonMessages)
{
	ETaskState ResultState = ETaskState::Unknown;
	FString PayloadKey = FetchMeshPayloadCommand.GetPayloadKey();
	FTransform MeshGlobalTransform = FetchMeshPayloadCommand.GetMeshGlobalTransform();
	FString ResultPayloadsUniqueId = FbxParser.FetchMeshPayload(PayloadKey, MeshGlobalTransform, ResultFolder);
	FJsonLoadSourceCmd::JsonResultParser ResultParser;
	ResultParser.SetResultFilename(FbxParser.GetResultPayloadFilepath(ResultPayloadsUniqueId));
	OutJSonMessages = FbxParser.GetJsonLoadMessages();
	OutJSonResult = ResultParser.ToJson();
	ResultState = ETaskState::ProcessOk;
	return ResultState;
}

ETaskState FInterchangeWorkerImpl::FetchFbxPayload(const FJsonFetchAnimationQueriesCmd& FetchAnimationBakeTransformPayloadCommand, FString& OutJSonResult, TArray<FString>& OutJSonMessages)
{
	ETaskState ResultState = ETaskState::Unknown;

	TMap<FString, FString> ResultPayloadFilePaths = FbxParser.FetchAnimationBakeTransformPayloads(FetchAnimationBakeTransformPayloadCommand.GetQueriesJsonString(), ResultFolder);
	
	FJsonFetchAnimationQueriesCmd::JsonAnimationQueriesResultParser ResultParser;
	ResultParser.SetHashToFilenames(ResultPayloadFilePaths);
	OutJSonMessages = FbxParser.GetJsonLoadMessages();
	OutJSonResult = ResultParser.ToJson();
	ResultState = ETaskState::ProcessOk;

	return ResultState;
}