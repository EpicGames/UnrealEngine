// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "FBX/InterchangeDispatcherFBXTasks.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformTime.h"
#include "HAL/Thread.h"
#include "InterchangeWorker.h"
#include "InterchangeCommands.h"
#include "InterchangeDispatcherNetworking.h"
#include "InterchangeFbxParser.h"

struct FFileStatData;
class FImportParameters;

class FInterchangeWorkerImpl
{
public:
	FInterchangeWorkerImpl(int32 InServerPID, int32 InServerPort, FString& InResultFolder);
	bool Run(const FString& WorkerVersionError);

private:
	void InitiatePing();
	void ProcessCommand(const UE::Interchange::FPingCommand& PingCommand);
	void ProcessCommand(const UE::Interchange::FBackPingCommand& BackPingCommand);
	void ProcessCommand(const TSharedPtr<UE::Interchange::ICommand> Command, const FString& ThreadName);
	void ProcessCommand(const UE::Interchange::FQueryTaskProgressCommand& QueryTaskProgressCommand);

	UE::Interchange::ETaskState LoadFbxFile(const UE::Interchange::FJsonFBXLoadSourceCmd& LoadSourceCommand, FString& OutJSonResult, TArray<FString>& OutJSonMessages);
	UE::Interchange::ETaskState FetchFbxPayload(const UE::Interchange::FJsonFetchPayloadCmd& FetchPayloadCommand, FString& OutJSonResult, TArray<FString>& OutJSonMessages);
	UE::Interchange::ETaskState FetchFbxPayload(const UE::Interchange::FJsonFetchMeshPayloadCmd& FetchMeshPayloadCommand, FString& OutJSonResult, TArray<FString>& OutJSonMessages);
	UE::Interchange::ETaskState FetchFbxPayload(const UE::Interchange::FJsonFetchAnimationQueriesCmd& FetchAnimationBakeTransformPayloadCommand, FString& OutJSonResult, TArray<FString>& OutJSonMessages);
private:
	UE::Interchange::FNetworkClientNode NetworkInterface;
	UE::Interchange::FCommandQueue CommandIO;

	int32 ServerPID;
	int32 ServerPort;
	uint64 PingStartCycle;
	FString ResultFolder;
	FCriticalSection TFinishThreadCriticalSection;
	TArray<FString> CurrentFinishThreads;
	TMap<FString, TFuture<bool> > ActiveThreads;

	UE::Interchange::FInterchangeFbxParser FbxParser;

};
